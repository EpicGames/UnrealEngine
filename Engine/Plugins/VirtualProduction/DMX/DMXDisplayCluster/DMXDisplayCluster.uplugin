{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DMX DisplayCluster",
	"Description": "Allows integration between DMX and DisplayCluster",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Modules": [
		{
			"Name": "DMXDisplayCluster",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DMXDisplayClusterLightCard",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "DMXEngine",
			"Enabled": true
		},
		{
			"Name": "DMXProtocol",
			"Enabled": true
		},
		{
			"Name": "nDisplay",
			"Enabled": true
		},
		{
			"Name": "nDisplayModularFeatures",
			"Enabled": true
		}
	]
}