// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXDisplayCluster : ModuleRules
{
	public DMXDisplayCluster(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[] { });

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"DMXRuntime",
			"DMXProtocol",
			"DisplayCluster"
		});

	}
}
