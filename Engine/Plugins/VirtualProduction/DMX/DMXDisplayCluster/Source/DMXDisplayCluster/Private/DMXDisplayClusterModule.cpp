// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXDisplayClusterModule.h"

#include "DMXDisplayClusterReplicator.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"


void FDMXDisplayClusterModule::StartupModule()
{
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().AddRaw(this, &FDMXDisplayClusterModule::CreateDMXDisplayClusterReplicator);
}

void FDMXDisplayClusterModule::CreateDMXDisplayClusterReplicator()
{
	// Only run when operating as cluster and not in editor
	if (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		if (IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr())
		{
			DMXDisplayClusterReplicator = MakeShared<FDMXDisplayClusterReplicator>();
		}
	}
}

void FDMXDisplayClusterModule::ShutdownModule()
{}

IMPLEMENT_MODULE(FDMXDisplayClusterModule, DMXDisplayCluster)
