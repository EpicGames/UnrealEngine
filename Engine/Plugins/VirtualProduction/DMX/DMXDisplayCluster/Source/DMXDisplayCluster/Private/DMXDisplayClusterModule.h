// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FDMXDisplayClusterReplicator;


class DMXDISPLAYCLUSTER_API FDMXDisplayClusterModule 
	: public IModuleInterface
{

public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

protected:
	/** Setup the DisplayCluster replicator */
	void CreateDMXDisplayClusterReplicator();

	/** The replicator instance */
	TSharedPtr<FDMXDisplayClusterReplicator> DMXDisplayClusterReplicator;
};
