// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXDisplayClusterReplicator.h"

#include "IO/DMXPortManager.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXRawListener.h"

#include "IDisplayCluster.h"
#include "Cluster/DisplayClusterClusterEvent.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/MemoryReader.h"


namespace
{
	/**
	 * Event ID for the Cluster event used to replicate DMX for nDisplay
	 *
	 * As per 4.26 there is no registry for system events, hence
	 * 0xDDDD0000 is used for DMX ndisplay replication system event by agreement 4.26
	 */
	constexpr int32 DMXnDisplayReplicationEventID = 0xDDDD0000;

	/** 
	 * A packet to replicate a dmx signal 
	 */
	struct FDMXDisplayClusterPacket
	{
		FDMXDisplayClusterPacket()
			: Signal(MakeShared<FDMXSignal, ESPMode::ThreadSafe>())
		{}

		FDMXDisplayClusterPacket(const FDMXSignalSharedPtr& InSignal, int32 InPortIndex)
			: Signal(InSignal)
			, PortIndex(InPortIndex)
		{}

		void Serialize(FArchive& Ar)
		{
			Signal->Serialize(Ar);

			Ar << PortIndex;
		}

		FDMXSignalSharedPtr Signal;

		/** Index of the port where the packet was received */
		int32 PortIndex = INDEX_NONE;
	};
}


FDMXDisplayClusterReplicator::FDMXDisplayClusterReplicator()
	: bClusterEventEmitter(false)
	, TickType(ETickableTickType::Never)
{

	// Only supported when running in cluster mode without editor.
	check(IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster);

	if (IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr())
	{
		const bool bForcePrimary = FParse::Param(FCommandLine::Get(), TEXT("dc_dmx_primary"));
		const bool bForceSecondary = FParse::Param(FCommandLine::Get(), TEXT("dc_dmx_secondary"));

		if (!ensureMsgf(!(bForcePrimary && bForceSecondary), TEXT("Ambigous command line for the DMX display cluster plugin. An instance cannot be 'dc_dmx_primary' and 'dc_dmx_secondary' at the same time.")))
		{
			return;
		}

		// Bind to OnClusterEventReceived. The DMX primary node needs to bind here too to get data at the same time as secondary nodes. 
		BinaryListener = FOnClusterEventBinaryListener::CreateRaw(this, &FDMXDisplayClusterReplicator::OnClusterEventReceived);
		ClusterManager->AddClusterEventBinaryListener(BinaryListener);

		// Cache the input ports. As the application runs in cluster mode without editor, the Input ports array is not expected to change.
		CachedInputPorts = FDMXPortManager::Get().GetInputPorts();

		bClusterEventEmitter = [ClusterManager, bForcePrimary, bForceSecondary]()
		{
			const bool bAutoMode = !bForcePrimary && !bForceSecondary;

			if (bAutoMode)
			{
				return ClusterManager->IsPrimary();
			}

			return bForcePrimary;
		}();

		if (bClusterEventEmitter)
		{
			TickType = ETickableTickType::Always;

			// User a custom listener to avoid issues with tick precedence between the port and this object's tick 
			for (const FDMXInputPortSharedRef& InputPort : CachedInputPorts)
			{
				// Don't use the default queue for the port, instead use a custom raw listener
				InputPort->SetUseDefaultQueue(false);

				TSharedPtr<FDMXRawListener> RawListener = MakeShared<FDMXRawListener>(InputPort);
				RawListener->Start();

				// Note we make sure here, and rely on the raw listener array is in the same order as the cached input ports array
				RawListeners.Add(RawListener);
			}
		}
		else
		{
			TickType = ETickableTickType::Never;

			// If this is not an emitter, suspend receiving from protocols to not get any data from the network.
			FDMXPortManager::Get().SuspendProtocols();
		}
	}
}

FDMXDisplayClusterReplicator::~FDMXDisplayClusterReplicator()
{
	// Release raw listeners
	for (const TSharedPtr<FDMXRawListener>& RawListener : RawListeners)
	{
		RawListener->Stop();
	}

	// Release binary listeners
	if (IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr())
	{
		if (BinaryListener.IsBound())
		{
			ClusterManager->RemoveClusterEventBinaryListener(BinaryListener);
		}
	}
}

void FDMXDisplayClusterReplicator::OnClusterEventReceived(const FDisplayClusterClusterEventBinary& Event)
{
	if (Event.EventId == DMXnDisplayReplicationEventID)
	{
		FMemoryReader Reader(Event.EventData);

		FDMXDisplayClusterPacket DMXDisplayClusterPacket;
		DMXDisplayClusterPacket.Serialize(Reader);

		if (ensureMsgf(CachedInputPorts.IsValidIndex(DMXDisplayClusterPacket.PortIndex), TEXT("FDMXDisplayClusterReplicator: Input Ports specified in project settings are not identical across the nDipslay cluster. DMX replication failed.")))
		{
			CachedInputPorts[DMXDisplayClusterPacket.PortIndex]->GameThreadInjectDMXSignal(DMXDisplayClusterPacket.Signal.ToSharedRef());
		}
	}
}

void FDMXDisplayClusterReplicator::Tick(float DeltaTime)
{
	check(bClusterEventEmitter);

	if (IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr())
	{
		for (int32 PortIndex = 0; PortIndex < CachedInputPorts.Num(); PortIndex++)
		{
			checkf(RawListeners.IsValidIndex(PortIndex), TEXT("The implementation relies on raw listeners corresponding to the ports array"));

			// Get all data from the raw listener, but only keep the latest signals of this tick.
			TSharedPtr<FDMXRawListener>& RawListener = RawListeners[PortIndex];

			ExternUniverseToSignalForReplicationMap.Reset();

			FDMXSignalSharedPtr Signal;
			int32 DummyLocalUniverseID; // Can be ignored, we need the extern universe
			while (RawListener->DequeueSignal(this, Signal, DummyLocalUniverseID))
			{
				ExternUniverseToSignalForReplicationMap.FindOrAdd(Signal->ExternUniverseID) = Signal;
			}

			// Emmit latest signals as cluster event
			for (const TTuple<int32, FDMXSignalSharedPtr>& ExternUniverseToSignalPair : ExternUniverseToSignalForReplicationMap)
			{
				FDMXDisplayClusterPacket DMXDisplayClusterPacket(ExternUniverseToSignalPair.Value, PortIndex);

				FArrayWriter ArrayWriter;
				DMXDisplayClusterPacket.Serialize(ArrayWriter);

				FDisplayClusterClusterEventBinary ClusterEvent;
				ClusterEvent.bIsSystemEvent = true;
				ClusterEvent.bShouldDiscardOnRepeat = false;
				ClusterEvent.EventId = DMXnDisplayReplicationEventID;
				ClusterEvent.EventData = MoveTemp(ArrayWriter);
				constexpr bool bEmitFromPrimaryOnly = false;
				ClusterManager->EmitClusterEventBinary(ClusterEvent, bEmitFromPrimaryOnly);
			}
		}
	}
}

ETickableTickType FDMXDisplayClusterReplicator::GetTickableTickType() const
{
	return TickType;
}

TStatId FDMXDisplayClusterReplicator::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDMXDisplayClusterReplicator, STATGROUP_Tickables);
}
