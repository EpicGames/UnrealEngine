// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "Tickable.h"
#include "Cluster/IDisplayClusterClusterManager.h"

class FDMXRawListener;


/** Replicates input ports on tick */
class DMXDISPLAYCLUSTER_API FDMXDisplayClusterReplicator
	: public FTickableGameObject
	, public TSharedFromThis<FDMXDisplayClusterReplicator>
{
public:
	FDMXDisplayClusterReplicator();

	virtual ~FDMXDisplayClusterReplicator();

private:
	/** Called whena cluster event was received */
	void OnClusterEventReceived(const FDisplayClusterClusterEventBinary& Event);

	//~Begin FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickableInEditor() const override { return true; }
	virtual ETickableTickType GetTickableTickType() const override;
	virtual TStatId GetStatId() const override;
	//~End FTickableGameObject interface

	/** True if it is a cluster event emitter, else it is a cluster event receiver */
	bool bClusterEventEmitter;

	/** The tick type used (always for emitter, never for listener) */
	ETickableTickType TickType;

	/** Listen for DisplayCluster DMX events */
	FOnClusterEventBinaryListener BinaryListener;

	/** Input porst currently in use */
	TArray<FDMXInputPortSharedRef> CachedInputPorts;

	/** Raw listeners for the input ports */
	TArray<TSharedPtr<FDMXRawListener>> RawListeners;

	/** Map of latest Singals that needs be replicated per extern Universe. Used to only replicate latest data on tick. */
	TMap<int32, FDMXSignalSharedPtr> ExternUniverseToSignalForReplicationMap;
};
