// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXDisplayClusterLightCard : ModuleRules
{
	public DMXDisplayClusterLightCard(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
				
		PublicDependencyModuleNames.AddRange(new string[]
		{                   
			"DisplayCluster",
			"DisplayClusterLightCardExtender",
			"DMXProtocol",
			"DMXRuntime",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
		});
	}
}
