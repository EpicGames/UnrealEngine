// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXDisplayClusterLightCardComponent.h"

#include "IO/DMXInputPort.h"
#include "IO/DMXPortManager.h"
#include "Library/DMXEntityFixturePatch.h"

#include "Serialization/ArrayReader.h"


/** Attribute names that match the current version of the DisplayClusterLightCard GDTF */
namespace UE::DMX::DMXDisplayClusterLightCardComponent::Private
{
	namespace AttributeNames
	{
		const FName Attribute_DMXInput = "DMXInput";
		const FName Attribute_DistanceFromCenter = "DistanceFromCenter";
		const FName Attribute_Pan = "Pan";
		const FName Attribute_Tilt = "Tilt";
		const FName Attribute_RotX = "Rot_X";
		const FName Attribute_RotY = "Rot_Y";
		const FName Attribute_RotZ = "Rot_Z";
		const FName Attribute_ScaleX = "Scale_X";
		const FName Attribute_ScaleY = "Scale_Y";
		const FName Attribute_Mask = "Mask";
		const FName Attribute_Red = "Red";
		const FName Attribute_Green = "Green";
		const FName Attribute_Blue = "Blue";
		const FName Attribute_ColorAdd_Alpha = "ColorAdd_Alpha";
		const FName Attribute_CTC = "CTC";
		const FName Attribute_Tint = "Tint";
		const FName Attribute_Exposure = "Exposure";
		const FName Attribute_Gain = "Gain";
		const FName Attribute_Opacity = "Opacity";
		const FName Attribute_Feathering = "Feathering";
		const FName Attribute_Alpha_Gradient_Enable = "Alpha_Gradient_Enable";
		const FName Attribute_StartingAlpha = "StartingAlpha";
		const FName Attribute_EndingAlpha = "EndingAlpha";
		const FName Attribute_Gradient_Angle = "Gradient_Angle";
	}
};


/** Actor data that can possibly be set to the light card actor */
struct FDMXDisplayClusterLightCardActorData
{
	void Apply(ADisplayClusterLightCardActor* Actor)
	{
		if (!Actor || Actor->IsProxy())
		{
			return;
		}

		// Early out if DMX Input is disabled
		if (!bDMXInput.IsSet() || !bDMXInput.GetValue())
		{
			return;
		}

		bool bNeedsUpdateLightCardTransform = false;
		bool bNeedsUpdateLightCardMaterialInstance = false;
		bool bNeedsUpdatePolygonTexture = false;

		if (DistanceFromCenter.IsSet() && Actor->DistanceFromCenter != DistanceFromCenter)
		{
			Actor->DistanceFromCenter = DistanceFromCenter.GetValue();
			bNeedsUpdateLightCardTransform = true;
		}

		if (Actor->bIsUVLightCard &&
			Longitude.IsSet() && Latitude.IsSet() &&
			(Actor->UVCoordinates.X != Longitude.GetValue() || Actor->UVCoordinates.Y != Latitude.GetValue()))
		{
			// Set UV Coordinates if in UV mode
			Actor->UVCoordinates = FVector2D(Longitude.GetValue(), Latitude.GetValue());
			bNeedsUpdateLightCardTransform = true;
		}
		else
		{
			// Set actual Longitude and Latidude if not in UV Mode
			if (Longitude.IsSet() && Actor->Longitude != Longitude)
			{
				Actor->Longitude = Longitude.GetValue();
				bNeedsUpdateLightCardTransform = true;
			}

			if (Latitude.IsSet() && Actor->Latitude != Latitude)
			{
				Actor->Latitude = Latitude.GetValue();
				bNeedsUpdateLightCardTransform = true;
			}
		}

		if (Spin.IsSet() && Actor->Spin != Spin)
		{
			Actor->Spin = Spin.GetValue();
			bNeedsUpdateLightCardTransform = true;
		}

		if (Yaw.IsSet() && Actor->Yaw != Yaw)
		{
			Actor->Yaw = Yaw.GetValue();
			bNeedsUpdateLightCardTransform = true;
		}

		if (Pitch.IsSet() && Actor->Pitch != Pitch)
		{
			Actor->Pitch = Pitch.GetValue();
			bNeedsUpdateLightCardTransform = true;
		}

		if (ScaleX.IsSet() && Actor->Scale.X != ScaleX)
		{
			Actor->Scale.X = ScaleX.GetValue();
			bNeedsUpdateLightCardTransform = true;
		}

		if (ScaleY.IsSet() && Actor->Scale.Y != ScaleY)
		{
			Actor->Scale.Y = ScaleY.GetValue();
			bNeedsUpdateLightCardTransform = true;
		}

		if (Mask.IsSet() && Actor->Mask != Mask)
		{
			Actor->Mask = Mask.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (ColorR.IsSet() && Actor->Color.R != ColorR)
		{
			Actor->Color.R = ColorR.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (ColorG.IsSet() && Actor->Color.G != ColorG)
		{
			Actor->Color.G = ColorG.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (ColorB.IsSet() && Actor->Color.B != ColorB)
		{
			Actor->Color.B = ColorB.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (ColorA.IsSet() && Actor->Color.A != ColorA)
		{
			Actor->Color.A = ColorA.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (Temperature.IsSet() && Actor->Temperature != Temperature)
		{
			Actor->Temperature = Temperature.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (Tint.IsSet() && Actor->Tint != Tint)
		{
			Actor->Tint = Tint.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (Exposure.IsSet() && Actor->Exposure != Exposure)
		{
			Actor->Exposure = Exposure.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (Gain.IsSet() && Actor->Gain != Gain)
		{
			Actor->Gain = Gain.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (Opacity.IsSet() && Actor->Opacity != Opacity)
		{
			Actor->Opacity = Opacity.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (Feathering.IsSet() && Actor->Feathering != Feathering)
		{
			Actor->Feathering = Feathering.GetValue();
			bNeedsUpdatePolygonTexture = true;
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		if (bAlphaGradientEnable.IsSet() && Actor->AlphaGradient.bEnableAlphaGradient != bAlphaGradientEnable)
		{
			Actor->AlphaGradient.bEnableAlphaGradient = bAlphaGradientEnable.GetValue();
			bNeedsUpdateLightCardMaterialInstance = true;
		}

		// Apply Alpha gradient only if Alpha Gradient Enable
		if (IsAlphaGradientEnabled())
		{
			if (StartingAlpha.IsSet() && Actor->AlphaGradient.StartingAlpha != StartingAlpha)
			{
				Actor->AlphaGradient.StartingAlpha = StartingAlpha.GetValue();
				bNeedsUpdateLightCardMaterialInstance = true;
			}

			if (EndingAlpha.IsSet() && Actor->AlphaGradient.EndingAlpha != EndingAlpha)
			{
				Actor->AlphaGradient.EndingAlpha = EndingAlpha.GetValue();
				bNeedsUpdateLightCardMaterialInstance = true;
			}

			if (GradientAngle.IsSet() && Actor->AlphaGradient.Angle != GradientAngle)
			{
				Actor->AlphaGradient.Angle = GradientAngle.GetValue();
				bNeedsUpdateLightCardMaterialInstance = true;
			}
		}

		// Apply changes
		if (bNeedsUpdateLightCardTransform)
		{
			Actor->UpdateStageActorTransform();

			// When the transform properties change and if bAlwaysFlushToWall is set, remain flush with the owning stage actor.
			if (Actor->bAlwaysFlushToWall)
			{
				Actor->MakeFlushToWall();
			}
		}

		if (bNeedsUpdateLightCardMaterialInstance)
		{
			Actor->UpdateLightCardMaterialInstance();
		}

		if (bNeedsUpdatePolygonTexture)
		{
			Actor->UpdatePolygonTexture();
		}

#if WITH_EDITOR
		if (bNeedsUpdateLightCardTransform || bNeedsUpdateLightCardMaterialInstance || bNeedsUpdatePolygonTexture)
		{
			// Broadcast a property changed event for the light card so that anything listening for light card property changes, such as the ICVFX panel,
			// can react appropriately. Since multiple properties may have changed, use an empty property event.
			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Actor, EmptyPropertyChangedEvent);
		}
#endif
	}

	void SetDMXInput(uint8 ShouldInputDMXValue)
	{
		bDMXInput = ShouldInputDMXValue > DMX_MAX_VALUE / 2;
	}

	void SetDistanceFromCenter(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		DistanceFromCenter = FMath::Lerp(ValueRanges.MinDistanceFromCenter, ValueRanges.MaxDistanceFromCenter, NormalizedValue);
	}

	void SetLongitude(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges, bool bIsUVLightCard)
	{
		if (bIsUVLightCard)
		{
			Longitude = FMath::Lerp(ValueRanges.MinLongitudeU, ValueRanges.MaxLongitudeU, NormalizedValue);
		}
		else
		{
			Longitude = FMath::Lerp(ValueRanges.MinLongitude, ValueRanges.MaxLongitude, NormalizedValue);
		}
	}

	void SetLatitude(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges, bool bIsUVLightCard)
	{
		if (bIsUVLightCard)
		{
			Latitude = FMath::Lerp(ValueRanges.MinLatitudeV, ValueRanges.MaxLatitudeV, NormalizedValue);
		}
		else
		{
			Latitude = FMath::Lerp(ValueRanges.MinLatitude, ValueRanges.MaxLatitude, NormalizedValue);
		}
	}

	void SetSpin(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Spin = FMath::Lerp(ValueRanges.MinSpin, ValueRanges.MaxSpin, NormalizedValue);
	}

	void SetPitch(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Pitch = FMath::Lerp(ValueRanges.MinPitch, ValueRanges.MaxPitch, NormalizedValue);
	}

	void SetYaw(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Yaw = FMath::Lerp(ValueRanges.MinYaw, ValueRanges.MaxYaw, NormalizedValue);
	}

	void SetScaleX(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		ScaleX = FMath::Lerp(ValueRanges.MinScale.X, ValueRanges.MaxScale.X, NormalizedValue);
	}
	
	void SetScaleY(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		ScaleY = FMath::Lerp(ValueRanges.MinScale.Y, ValueRanges.MaxScale.Y, NormalizedValue);
	}

	void SetMask(uint8 AbsoluteValue)
	{
		// Skip through if the value is 0
		if (AbsoluteValue == 0)
		{
			return;
		}

		// Scale the value to ranges of 5, so 1-5 is the first enum value, 6-10 the second, etc. 
		const uint8 AbsoluteEnumValue = (AbsoluteValue + 4) / 5 - 1;
		Mask = static_cast<EDisplayClusterLightCardMask>(FMath::Clamp(AbsoluteEnumValue, 0, static_cast<uint8>(EDisplayClusterLightCardMask::MAX) - 1));
	}

	void SetColorR(double NormalizedValue)
	{
		ColorR = NormalizedValue;;
	}

	void SetColorG(double NormalizedValue)
	{
		ColorG = NormalizedValue;;
	}

	void SetColorB(double NormalizedValue)
	{
		ColorB = NormalizedValue;;
	}

	void SetColorA(double NormalizedValue)
	{
		ColorA = NormalizedValue;;
	}

	void SetTemperature(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Temperature = FMath::Lerp(ValueRanges.MinTemperature, ValueRanges.MaxTemperature, NormalizedValue);
	}

	void SetTint(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Tint = FMath::Lerp(ValueRanges.MinTint, ValueRanges.MaxTint, NormalizedValue);
	}

	void SetExposure(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Exposure = FMath::Lerp(ValueRanges.MinExposure, ValueRanges.MaxExposure, NormalizedValue);
	}

	void SetGain(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Gain = FMath::Lerp(ValueRanges.MinGain, ValueRanges.MaxGain, NormalizedValue);
	}

	void SetOpacity(double NormalizedValue)
	{
		Opacity = NormalizedValue;
	}

	void SetFeathering(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		Feathering = FMath::Lerp(ValueRanges.MinFeathering, ValueRanges.MaxFeathering, NormalizedValue);
	}

	void SetAlphaGradientEnable(uint8 AlphaGradientEnableValue)
	{
		bAlphaGradientEnable = AlphaGradientEnableValue > DMX_MAX_VALUE / 2;
	}

	void SetStartingAlpha(double NormalizedValue)
	{
		StartingAlpha = NormalizedValue;
	}

	void SetEndingAlpha(double NormalizedValue)
	{
		EndingAlpha = NormalizedValue;
	}

	void SetGradientAngle(double NormalizedValue, const FDMXDisplayClusterLightCardActorDataValueRanges& ValueRanges)
	{
		GradientAngle = FMath::Lerp(ValueRanges.MinGradientAngle, ValueRanges.MaxGradientAngle, NormalizedValue);
	}

	bool IsDMXInputEnabled() const { return bDMXInput.IsSet() && bDMXInput.GetValue(); }
	bool IsAlphaGradientEnabled() const { return bAlphaGradientEnable.IsSet() && bAlphaGradientEnable.GetValue(); }

private:
	TOptional<bool> bDMXInput;
	TOptional<double> DistanceFromCenter;
	TOptional<double> Longitude;
	TOptional<double> Latitude;
	TOptional<double> Spin;
	TOptional<double> Pitch;
	TOptional<double> Yaw;
	TOptional<double> ScaleX;
	TOptional<double> ScaleY;
	TOptional<EDisplayClusterLightCardMask> Mask;
	TOptional<float> ColorR;
	TOptional<float> ColorG;
	TOptional<float> ColorB;
	TOptional<float> ColorA;
	TOptional<float> Temperature;
	TOptional<float> Tint;
	TOptional<float> Exposure;
	TOptional<float> Gain;
	TOptional<float> Opacity;
	TOptional<float> Feathering;
	TOptional<bool> bAlphaGradientEnable;
	TOptional<float> StartingAlpha;
	TOptional<float> EndingAlpha;
	TOptional<float> GradientAngle;
};

void UDMXDisplayClusterLightCardComponent::OnRegister()
{
	Super::OnRegister();

	if (!OnFixturePatchReceived.IsBound())
	{
		OnFixturePatchReceived.AddDynamic(this, &UDMXDisplayClusterLightCardComponent::OnLightCardReceivedDMXFromPatch);
	}
}

void UDMXDisplayClusterLightCardComponent::PostLoad()
{
	Super::PostLoad();

	if (!OnFixturePatchReceived.IsBound())
	{
		OnFixturePatchReceived.AddDynamic(this, &UDMXDisplayClusterLightCardComponent::OnLightCardReceivedDMXFromPatch);
	}
}

#if WITH_EDITOR
void UDMXDisplayClusterLightCardComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Enable tick if no fixture patch is set and vice versa
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDMXComponent, FixturePatchRef))
	{
		if (FixturePatchRef.GetFixturePatch())
		{
			if (!OnFixturePatchReceived.IsBound())
			{
				OnFixturePatchReceived.AddDynamic(this, &UDMXDisplayClusterLightCardComponent::OnLightCardReceivedDMXFromPatch);
			}
		}
	}
}
#endif // WITH_EDITOR

void UDMXDisplayClusterLightCardComponent::OnLightCardReceivedDMXFromPatch(UDMXEntityFixturePatch* FixturePatch, const FDMXNormalizedAttributeValueMap& ValuePerAttribute)
{
	using namespace UE::DMX::DMXDisplayClusterLightCardComponent::Private::AttributeNames;

	ADisplayClusterLightCardActor* Actor = Cast<ADisplayClusterLightCardActor>(GetOwner());
	if (!Actor)
	{
		return;
	}

	bool bAccessDMXInputValueSuccess;
	const uint8 DMXInputValue = FixturePatch->GetAttributeValue(Attribute_DMXInput, bAccessDMXInputValueSuccess);
	if (!bAccessDMXInputValueSuccess)
	{
		// Don't proceed if DMXInput is not set
		return;
	}

	FDMXDisplayClusterLightCardActorData ActorData;
	ActorData.SetDMXInput(DMXInputValue);

	// Early out if DMX Input is disabled
	if (!ActorData.IsDMXInputEnabled())
	{
		return;
	}

	const float* const DistanceFromCenterPtr = ValuePerAttribute.Map.Find(Attribute_DistanceFromCenter);
	if (DistanceFromCenterPtr)
	{
		ActorData.SetDistanceFromCenter(*DistanceFromCenterPtr, ValueRanges);
	}

	const float* const LongitudePtr = ValuePerAttribute.Map.Find(Attribute_Pan);
	if (LongitudePtr)
	{
		ActorData.SetLongitude(*LongitudePtr, ValueRanges, Actor->bIsUVLightCard);
	}

	const float* const LatitudePtr = ValuePerAttribute.Map.Find(Attribute_Tilt);
	if (LatitudePtr)
	{
		ActorData.SetLatitude(*LatitudePtr, ValueRanges, Actor->bIsUVLightCard);
	}

	const float* const SpinPtr = ValuePerAttribute.Map.Find(Attribute_RotX);
	if (SpinPtr)
	{
		ActorData.SetSpin(*SpinPtr, ValueRanges);
	}

	const float* const PitchPtr = ValuePerAttribute.Map.Find(Attribute_RotY);
	if (PitchPtr)
	{
		ActorData.SetPitch(*PitchPtr, ValueRanges);
	}

	const float* const YawPtr = ValuePerAttribute.Map.Find(Attribute_RotZ);
	if (YawPtr)
	{
		ActorData.SetYaw(*YawPtr, ValueRanges);
	}

	const float* const ScaleXPtr = ValuePerAttribute.Map.Find(Attribute_ScaleX);
	if (ScaleXPtr)
	{
		ActorData.SetScaleX(*ScaleXPtr, ValueRanges);
	}

	const float* const ScaleYPtr = ValuePerAttribute.Map.Find(Attribute_ScaleY);
	if (ScaleYPtr)
	{
		ActorData.SetScaleY(*ScaleYPtr, ValueRanges);
	}

	bool bAccessMaskValueSuccess;
	const uint8 Mask = FixturePatch->GetAttributeValue(Attribute_Mask, bAccessMaskValueSuccess);
	if (bAccessMaskValueSuccess)
	{
		ActorData.SetMask(Mask);
	}

	const float* const RedPtr = ValuePerAttribute.Map.Find(Attribute_Red);
	if (RedPtr)
	{
		ActorData.SetColorR(*RedPtr);
	}

	const float* const GreenPtr = ValuePerAttribute.Map.Find(Attribute_Green);
	if (GreenPtr)
	{
		ActorData.SetColorG(*GreenPtr);
	}

	const float* const BluePtr = ValuePerAttribute.Map.Find(Attribute_Blue);
	if (BluePtr)
	{
		ActorData.SetColorB(*BluePtr);
	}

	const float* const AlphaPtr = ValuePerAttribute.Map.Find(Attribute_ColorAdd_Alpha);
	if (AlphaPtr)
	{
		ActorData.SetColorA(*AlphaPtr);
	}

	const float* const CTCPtr = ValuePerAttribute.Map.Find(Attribute_CTC);
	if (CTCPtr)
	{
		ActorData.SetTemperature(*CTCPtr, ValueRanges);
	}

	const float* const TintPtr = ValuePerAttribute.Map.Find(Attribute_Tint);
	if (TintPtr)
	{
		ActorData.SetTint(*TintPtr, ValueRanges);
	}

	const float* const ExposurePtr = ValuePerAttribute.Map.Find(Attribute_Exposure);
	if (ExposurePtr)
	{
		ActorData.SetExposure(*ExposurePtr, ValueRanges);
	}

	const float* const GainPtr = ValuePerAttribute.Map.Find(Attribute_Gain);
	if (GainPtr)
	{
		ActorData.SetGain(*GainPtr, ValueRanges);
	}

	const float* const OpacityPtr = ValuePerAttribute.Map.Find(Attribute_Opacity);
	if (OpacityPtr)
	{
		ActorData.SetOpacity(*OpacityPtr);
	}

	const float* const FeatheringPtr = ValuePerAttribute.Map.Find(Attribute_Feathering);
	if (FeatheringPtr)
	{
		ActorData.SetFeathering(*FeatheringPtr, ValueRanges);
	}

	bool bAccessAlphaGradientEnableValueSuccess;
	const int32 AlphaGradientEnableValue = FixturePatch->GetAttributeValue(Attribute_Alpha_Gradient_Enable, bAccessAlphaGradientEnableValueSuccess);
	if (bAccessAlphaGradientEnableValueSuccess)
	{
		ActorData.SetAlphaGradientEnable(AlphaGradientEnableValue);
	}

	if (ActorData.IsAlphaGradientEnabled())
	{
		const float* const StartingAlphaPtr = ValuePerAttribute.Map.Find(Attribute_StartingAlpha);
		if (StartingAlphaPtr)
		{
			ActorData.SetStartingAlpha(*StartingAlphaPtr);
		}

		const float* const EndingAlphaPtr = ValuePerAttribute.Map.Find(Attribute_EndingAlpha);
		if (EndingAlphaPtr)
		{
			ActorData.SetEndingAlpha(*EndingAlphaPtr);
		}

		const float* const GradientAnglePtr = ValuePerAttribute.Map.Find(Attribute_Gradient_Angle);
		if (GradientAnglePtr)
		{
			ActorData.SetGradientAngle(*GradientAnglePtr, ValueRanges);
		}
	}

	ActorData.Apply(Actor);
}
