// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterLightCardActor.h"
#include "IO/DMXInputPortReference.h"
#include "Game/DMXComponent.h"

#include "Components/ActorComponent.h"

#include "DMXDisplayClusterLightCardComponent.generated.h"

class UDMXEntityFixturePatch;



USTRUCT()
struct FDMXDisplayClusterLightCardActorDataValueRanges
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MinDistanceFromCenter = 0.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MaxDistanceFromCenter = 1000.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MinLongitude = 0.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MaxLongitude = 360.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MinLatitude = -90.0;
	
	UPROPERTY(EditAnywhere, Category = "DMX")
	double MaxLatitude = 90.0;

	/** Min longitude when the light card uses UV Mode */
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinLongitudeU = -2.f;
	
	/** Max longitude when the light card uses UV Mode */
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxLongitudeU = 2.f;
		
	/** Min latitude when the light card uses UV Mode */
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinLatitudeV = -2.f;
	
	/** Max latitude when the light card uses UV Mode */
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxLatitudeV = 2.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MinSpin = 0.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MaxSpin = 360.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MinPitch = 0.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MaxPitch = 360.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MinYaw = 0.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	double MaxYaw = 360.0;

	UPROPERTY(EditAnywhere, Category = "DMX")
	FVector2D MinScale = FVector2D(0.f, 0.f);

	UPROPERTY(EditAnywhere, Category = "DMX")
	FVector2D MaxScale = FVector2D(5.f, 5.f);

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinTemperature = 0.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxTemperature = 10000.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinTint = -1.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxTint = 1.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinExposure = -5.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxExposure = 5.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinGain = 0.f;
	
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxGain = 5.f;
	
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinFeathering = 0.f;
	
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxFeathering = 3.f;

	UPROPERTY(EditAnywhere, Category = "DMX")
	float MinGradientAngle = 0.f;
	
	UPROPERTY(EditAnywhere, Category = "DMX")
	float MaxGradientAngle = 360.f;
};

UCLASS(NotBlueprintable, HideCategories = (AssetUserData, Tags, Replication))
class UDMXDisplayClusterLightCardComponent
	: public UDMXComponent
{
	GENERATED_BODY()

protected:
	//~ Begin UObject interface
	virtual void OnRegister() override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

public:
	/** Value ranges for Min and Max DMX Values */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "DMX", Meta = (ShowOnlyInnerProperties))
	FDMXDisplayClusterLightCardActorDataValueRanges ValueRanges;

private:
	/** Called when the fixture patch received DMX */
	UFUNCTION()
	void OnLightCardReceivedDMXFromPatch(UDMXEntityFixturePatch* FixturePatch, const FDMXNormalizedAttributeValueMap& ValuePerAttribute);
};
