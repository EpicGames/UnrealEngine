// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXDisplayClusterLightCardModule.h"

#include "DMXDisplayClusterLightCardComponent.h"

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"


void FDMXDisplayClusterLightCardModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(IDisplayClusterLightCardActorExtender::ModularFeatureName, this);
}

void FDMXDisplayClusterLightCardModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IDisplayClusterLightCardActorExtender::ModularFeatureName, this);
}

FName FDMXDisplayClusterLightCardModule::GetExtenderName() const
{
	return "DMXDisplayClusterLightCardExtender";
}

TSubclassOf<UActorComponent> FDMXDisplayClusterLightCardModule::GetAdditionalSubobjectClass()
{
	return UDMXDisplayClusterLightCardComponent::StaticClass();
}

#if WITH_EDITOR
FName FDMXDisplayClusterLightCardModule::GetCategory() const
{
	return "DMX";
}
#endif

#if WITH_EDITOR
bool FDMXDisplayClusterLightCardModule::ShouldShowSubcategories() const
{
	return false;
}
#endif

IMPLEMENT_MODULE(FDMXDisplayClusterLightCardModule, DMXDisplayClusterLightCard);
