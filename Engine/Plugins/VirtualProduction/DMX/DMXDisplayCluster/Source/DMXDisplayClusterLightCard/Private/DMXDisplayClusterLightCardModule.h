// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterLightCardActorExtender.h"

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"


class FDMXDisplayClusterLightCardModule 
	: public IModuleInterface
	, public IDisplayClusterLightCardActorExtender
{
public:
	//~ Begin IModuleInterface inteface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface inteface

	//~ Begin IDisplayClusterLightCardActorExtender interface
	virtual FName GetExtenderName() const override;
	virtual TSubclassOf<UActorComponent> GetAdditionalSubobjectClass()override;
#if WITH_EDITOR
	virtual FName GetCategory() const override;
	virtual bool ShouldShowSubcategories() const override;
#endif
	//~ End IDisplayClusterLightCardActorExtender interface
};
